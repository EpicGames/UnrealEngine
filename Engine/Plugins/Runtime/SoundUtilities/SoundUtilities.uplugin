{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Sound Utilities",
	"Description" : "A variety of BP functions, objects, and utilities for audio.",
	"Category" : "Audio",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "SoundUtilities",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "SoundUtilitiesEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}