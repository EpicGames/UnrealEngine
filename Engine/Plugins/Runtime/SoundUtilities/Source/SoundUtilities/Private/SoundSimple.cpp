// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundSimple.h"
#include "ActiveSound.h"
#include "Sound/SoundWave.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundSimple)

void USoundSimple::CacheValues()
{
	MaxDistance = 0.0f;
	Duration = 0.0f;

	for (int32 i = 0; i < Variations.Num(); ++i)
	{
		USoundWave* SoundWaveVariation = Variations[i].SoundWave;
		if (SoundWaveVariation)
		{
			float SoundWaveMaxAudibleDistance = SoundWaveVariation->GetMaxDistance();
			if (SoundWaveMaxAudibleDistance > MaxDistance)
			{
				MaxDistance = SoundWaveMaxAudibleDistance;
			}

			float SoundWaveMaxDuration = SoundWaveVariation->GetDuration();
			if (SoundWaveMaxDuration > Duration)
			{
				Duration = SoundWaveMaxDuration;
			}
		}
	}
}

void USoundSimple::PostLoad()
{
	Super::PostLoad();

	CacheValues();
}

void USoundSimple::Serialize(FArchive& Ar)
{
	// Always force the duration to be updated when we are saving or cooking
	if (Ar.IsSaving() || Ar.IsCooking())
	{
		CacheValues();
	}

	Super::Serialize(Ar);
}

bool USoundSimple::IsPlayable() const
{
	return true;
}

void USoundSimple::Parse(class FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances)
{
	FWaveInstance* WaveInstance = ActiveSound.FindWaveInstance(NodeWaveInstanceHash);
	if (!WaveInstance)
	{
		ChooseSoundWave();
	}

	if (!SoundWave)
	{
		return;
	}

	// Forward the parse to the chosen sound wave
	SoundWave->Parse(AudioDevice, NodeWaveInstanceHash, ActiveSound, ParseParams, WaveInstances);
}

void USoundSimple::ChooseSoundWave()
{
	if (!Variations.Num())
	{
		return;
	}

	float ProbabilitySum = 0.0f;
	for (int32 i = 0; i < Variations.Num(); ++i)
	{
		if (Variations[i].SoundWave)
		{
			ProbabilitySum += Variations[i].ProbabilityWeight;
		}
	}

	float Choice = FMath::FRandRange(0.0f, ProbabilitySum);

	// Find the index chosen
	ProbabilitySum = 0.0f;
	int32 ChosenIndex = 0;
	for (int32 i = 0; i < Variations.Num(); ++i)
	{
		if (Variations[i].SoundWave)
		{
			float NextSum = ProbabilitySum + Variations[i].ProbabilityWeight;

			if (Choice >= ProbabilitySum && Choice < NextSum)
			{
				ChosenIndex = i;
				break;
			}

			ProbabilitySum = NextSum;
		}
	}
	
	check(ChosenIndex < Variations.Num());
	FSoundVariation& SoundVariation = Variations[ChosenIndex];

	check(SoundVariation.SoundWave);

	// Now choise the volume and pitch to use based on prob ranges
	float Volume = FMath::FRandRange(SoundVariation.VolumeRange[0], SoundVariation.VolumeRange[1]);
	float Pitch = FMath::FRandRange(SoundVariation.PitchRange[0], SoundVariation.PitchRange[1]);

	// Assign the sound wave value to the transient sound wave ptr
	SoundWave = SoundVariation.SoundWave;
	SoundWave->Volume = Volume;
	SoundWave->Pitch = Pitch;
}

float USoundSimple::GetMaxDistance() const
{
	return MaxDistance;
}

float USoundSimple::GetDuration() const
{
	return Duration;
}
