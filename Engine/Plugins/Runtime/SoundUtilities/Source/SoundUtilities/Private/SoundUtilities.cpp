// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundUtilities.h"
#include "DSP/Dsp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundUtilities)

static constexpr int32 MIDI_MIN = 0;
static constexpr int32 MIDI_MAX = 127;

float USoundUtilitiesBPFunctionLibrary::GetBeatTempo(float BeatsPerMinute, int32 BeatMultiplier, int32 DivisionsOfWholeNote)
{
	const float QuarterNoteTime = 60.0f / FMath::Max(1.0f, BeatsPerMinute);
	return 4.0f * (float)BeatMultiplier * QuarterNoteTime / FMath::Max(1, DivisionsOfWholeNote);
}

float USoundUtilitiesBPFunctionLibrary::GetFrequencyFromMIDIPitch(const int32 MidiNote)
{
	return Audio::GetFrequencyFromMidi(FMath::Clamp(MidiNote, MIDI_MIN, MIDI_MAX));
}

int32 USoundUtilitiesBPFunctionLibrary::GetMIDIPitchFromFrequency(const float Frequency)
{
	return FMath::Clamp((int32)Audio::GetMidiFromFrequency(Frequency), MIDI_MIN, MIDI_MAX);
}

float USoundUtilitiesBPFunctionLibrary::GetPitchScaleFromMIDIPitch(const int32 BaseMidiNote, const int32 TargetMidiNote)
{
	return Audio::GetPitchScaleFromMIDINote(FMath::Clamp(BaseMidiNote, MIDI_MIN, MIDI_MAX), FMath::Clamp(TargetMidiNote, MIDI_MIN, MIDI_MAX));
}

float USoundUtilitiesBPFunctionLibrary::GetGainFromMidiVelocity(int InVelocity)
{
	return Audio::GetGainFromVelocity(FMath::Clamp(InVelocity, MIDI_MIN, MIDI_MAX));
}

float USoundUtilitiesBPFunctionLibrary::ConvertLinearToDecibels(const float InLinear, const float InFloor)
{
	return Audio::ConvertToDecibels(InLinear, FMath::Max(SMALL_NUMBER, InFloor));
}

float  USoundUtilitiesBPFunctionLibrary::ConvertDecibelsToLinear(const float InDecibels)
{
	return Audio::ConvertToLinear(InDecibels);
}

float USoundUtilitiesBPFunctionLibrary::GetLogFrequencyClamped(const float InValue, const FVector2D& InDomain, const FVector2D& InRange)
{
	FVector2D ClampedRange = InRange.ClampAxes(SMALL_NUMBER, BIG_NUMBER);

	return Audio::GetLogFrequencyClamped(InValue, InDomain, ClampedRange);
}

float USoundUtilitiesBPFunctionLibrary::GetLinearFrequencyClamped(const float InValue, const FVector2D& InDomain, const FVector2D& InRange)
{
	FVector2D ClampedRange = InRange.ClampAxes(SMALL_NUMBER, BIG_NUMBER);

	return Audio::GetLinearFrequencyClamped(InValue, InDomain, InRange);
}

float USoundUtilitiesBPFunctionLibrary::GetFrequencyMultiplierFromSemitones(const float InPitchSemitones)
{
	return Audio::GetFrequencyMultiplier(InPitchSemitones);
}

float USoundUtilitiesBPFunctionLibrary::GetBandwidthFromQ(const float InQ)
{
	return Audio::GetBandwidthFromQ(InQ);
}

float USoundUtilitiesBPFunctionLibrary::GetQFromBandwidth(const float InBandwidth)
{
	return Audio::GetQFromBandwidth(InBandwidth);
}
