// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundUtilitiesModule.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogSoundUtilities);

IMPLEMENT_MODULE(FSoundUtilitiesModule, SoundUtilities)

void FSoundUtilitiesModule::StartupModule()
{
}

void FSoundUtilitiesModule::ShutdownModule()
{
}


