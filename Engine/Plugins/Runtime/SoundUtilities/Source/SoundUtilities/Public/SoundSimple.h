// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SoundUtilitiesModule.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundBase.h"
#include "SoundSimple.generated.h"

#define UE_API SOUNDUTILITIES_API

USTRUCT(BlueprintType)
struct FSoundVariation
{
	GENERATED_USTRUCT_BODY()

	// The sound wave asset to use for this variation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SoundVariation")
	TObjectPtr<USoundWave> SoundWave;

	// The probability weight to use for this variation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	float ProbabilityWeight;

	// The volume range to use for this variation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	FVector2D VolumeRange;

	// The pitch range to use for this variation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	FVector2D PitchRange;

	FSoundVariation()
		: SoundWave(nullptr)
		, ProbabilityWeight(1.0f)
		, VolumeRange(1.0f, 1.0f)
		, PitchRange(1.0f, 1.0f)
	{
	}
};

// Class which contains a simple list of sound wave variations
UCLASS(MinimalAPI, ClassGroup = Sound, meta = (BlueprintSpawnableComponent))
class USoundSimple : public USoundBase
{
	GENERATED_BODY()

public:

	// List of variations for the simple sound
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Variations")
	TArray<FSoundVariation> Variations;

	//~ Begin UObject Interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject INterface

	//~ Begin USoundBase Interface.
	UE_API virtual bool IsPlayable() const override;
	UE_API virtual void Parse(class FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances) override;
	UE_API virtual float GetMaxDistance() const override;
	UE_API virtual float GetDuration() const override;
	//~ End USoundBase Interface.

protected:
	UE_API void ChooseSoundWave();
	UE_API void CacheValues();

	// The current chosen sound wave
	UPROPERTY(transient)
	TObjectPtr<USoundWave> SoundWave;
};



#undef UE_API
