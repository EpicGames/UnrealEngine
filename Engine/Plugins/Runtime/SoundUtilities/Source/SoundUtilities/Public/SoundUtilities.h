// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "SoundUtilities.generated.h"

/** Sound Utilities Blueprint Function Library
*  A library of Sound related functions for use in Blueprints
*/
UCLASS(MinimalAPI)
class USoundUtilitiesBPFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	/** Calculates a beat time in seconds from the given BPM, beat multiplier and divisions of a whole note. */
	UFUNCTION(BlueprintCallable, Category = "SoundUtilitiesBPLibrary")
	static float GetBeatTempo(float BeatsPerMinute = 120.0f, int32 BeatMultiplier = 1, int32 DivisionsOfWholeNote = 4);

	/** 
	 * Calculates Frequency values based on MIDI Pitch input
	 * @param MidiNote	The MIDI note to calculate the frequency of
	 * @return			The frequency in Hz that corresponds to the MIDI input
	 */
	UFUNCTION(BlueprintCallable, Category = "SoundUtilitiesBPLibrary")
	static float GetFrequencyFromMIDIPitch(const int32 MidiNote);

	/** 
	 * Calculates MIDI Pitch values based on frequency input
	 * @param Frequency		The frequency in Hz to convert into MIDI
	 * @return				The MIDI note closest to the inputted frequency
	 */
	UFUNCTION(BlueprintCallable, Category = "SoundUtilitiesBPLibrary")
	static int32 GetMIDIPitchFromFrequency(const float Frequency);

	/** 
	 * Calculates Pitch Scalar based on starting frequency and desired MIDI Pitch output 
	 * @param BaseMidiNote		The MIDI note corresponding to the starting frequency
	 * @param TargetMidiNote	The MIDI note corresponding to the desired final frequency
	 * @return					The amount to scale the pitch of the base note by, in order
	 *							for its pitch to match the target MIDI note
	*/
	UFUNCTION(BlueprintCallable, Category = "SoundUtilitiesBPLibrary")
	static float GetPitchScaleFromMIDIPitch(const int32 BaseMidiNote, const int32 TargetMidiNote);

	/** 
	 * Given a velocity value [0,127], return the linear gain 
	 * @param InVelocity	The MIDI velocity value to calculate the gain of
	 * @return				The gain corresponding to the MIDI value
	 */
	UFUNCTION(BlueprintCallable, Category = "SoundUtilitiesBPLibrary")
	static float GetGainFromMidiVelocity(int InVelocity);

	/** 
	 * Converts linear scale volume to decibels 
	 * @param InLinear					The linear scalar value to convert to decibels
	 * @param InFloor					The floor value to check against
	 * @return							The decibel value of the inputted linear scale
	*/
	UFUNCTION(BlueprintCallable, Category = "SoundUtilitiesBPLibrary", meta = (AdvancedDisplay = 1))
	static float ConvertLinearToDecibels(const float InLinear, const float InFloor);

	/** 
	 * Converts decibel to linear scale 
	 * @param InDecibels	The decibels to convert to a linear gain scalar
	 * @return				The resulting linear gain
	 */
	UFUNCTION(BlueprintCallable, Category = "SoundUtilitiesBPLibrary")
	static float ConvertDecibelsToLinear(const float InDecibels);

	/** 
	 * Returns the log frequency of the input value. Maps linear domain and range values to log output (good for linear slider controlling frequency) 
	 * @param InValue					The linear value to convert to logarithmic frequency
	 * @param InDomain					The domain to use when converting between linear and logarithmic scales
	 * @param InRange					The range to use when converting between linear and logarithmic scales
	 * @return							The log frequency of the given input
	 */
	UFUNCTION(BlueprintCallable, Category = "SoundUtilitiesBPLibrary")
	static float GetLogFrequencyClamped(const float InValue, const FVector2D& InDomain, const FVector2D& InRange);

	/**
	 * Returns the linear frequency of the input value. Maps log domain and range values to linear output (good for linear 
	 * slider representation/visualization of log frequency). Reverse of GetLogFrequencyClamped. 
	 * @param InValue					The logarithmic value to convert to linear frequency
	 * @param InDomain					The domain to use when converting between linear and logarithmic scales
	 * @param InRange					The range to use when converting between linear and logarithmic scales
	 * @return							The linear frequency of the given logarithmic input
	 */
	UFUNCTION(BlueprintCallable, Category = "SoundUtilitiesBPLibrary")
	static float GetLinearFrequencyClamped(const float InValue, const FVector2D& InDomain, const FVector2D& InRange);

	/**  
	* Returns the frequency multiplier to scale a base frequency given the input semitones 
	* @param InPitchSemitones	The amount of semitones to alter a frequency by
	* @return					The frequency multiplier that corresponds to the given change in semitones
	*/
	UFUNCTION(BlueprintCallable, Category = "SoundUtilitiesBPLibrary")
	static float GetFrequencyMultiplierFromSemitones(const float InPitchSemitones);

	/** 
	 * Helper function to get bandwidth from Q 
	 * @param InQ	The Q value to convert to bandwidth
	 * @return		The bandwidth value that corresponds to the given Q value
	 */
	UFUNCTION(BlueprintCallable, Category = "SoundUtilitiesBPLibrary")
	static float GetBandwidthFromQ(const float InQ);

	/**
	 * Helper function to get Q from bandwidth
	 * @param InBandwidth	The bandwidth value to convert to Q
	 * @return				The Q value that corresponds to the given bandwidth value
	 */
	UFUNCTION(BlueprintCallable, Category = "SoundUtilitiesBPLibrary", DisplayName = "Get Q From Bandwidth")
	static float GetQFromBandwidth(const float InBandwidth);
};
