// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SoundUtilities : ModuleRules
	{
        public SoundUtilities(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
				new string[] {
                    "Core",
					"CoreUObject",
					"Engine",
					"AudioMixer",
                    "UMG",
                    "Slate",
                    "SlateCore",
                    "InputCore",
                    "Projects"
                }
            );
		}
	}
}