// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions_SoundSimple.h"
#include "SoundSimple.h"

UClass* FAssetTypeActions_SoundSimple::GetSupportedClass() const
{
	return USoundSimple::StaticClass();
}

