// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions/AssetTypeActions_SoundBase.h"

class FAssetTypeActions_SoundSimple : public FAssetTypeActions_SoundBase
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SoundSimple", "Simple Sound"); }
	virtual FColor GetTypeColor() const override { return FColor(212, 97, 85); }
	virtual UClass* GetSupportedClass() const override;
	virtual bool CanFilter() override { return true; }
};
