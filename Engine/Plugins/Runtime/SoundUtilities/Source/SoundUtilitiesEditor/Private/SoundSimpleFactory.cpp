// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundSimpleFactory.h"
#include "AudioAnalytics.h"
#include "Sound/SoundWave.h"
#include "SoundSimple.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundSimpleFactory)

USoundSimpleFactory::USoundSimpleFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundSimple::StaticClass();

	bCreateNew = false;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundSimpleFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundSimple* SoundSimple = NewObject<USoundSimple>(InParent, Name, Flags);

	float Duration = 0.0f;

	// Add the sound waves with default variation entries
	for (int32 i = 0; i < SoundWaves.Num(); ++i)
	{
		FSoundVariation NewVariation;
		NewVariation.SoundWave = SoundWaves[i];

		if (SoundWaves[i]->Duration > Duration)
		{
			Duration = SoundWaves[i]->Duration;
		}

		SoundSimple->Variations.Add(NewVariation);
	}

	// Write out the duration to be the longest duration
	SoundSimple->Duration = Duration;

	Audio::Analytics::RecordEvent_Usage("SoundUtilities.SimpleSoundCreated");

	return SoundSimple;
}
