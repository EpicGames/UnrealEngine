// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "SoundSimpleFactory.generated.h"

class USoundWave;

UCLASS(hidecategories = Object, MinimalAPI)
class USoundSimpleFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

	/** Sound waves to create the simple sound with */
	UPROPERTY()
	TArray<TObjectPtr<USoundWave>> SoundWaves;
};

