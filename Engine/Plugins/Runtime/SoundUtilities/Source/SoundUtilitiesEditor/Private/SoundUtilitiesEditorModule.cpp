// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundUtilitiesEditorModule.h"
#include "AssetTypeActions_SoundSimple.h"
#include "ToolMenus.h"
#include "SoundWaveAssetActionExtender.h"

IMPLEMENT_MODULE(FSoundUtilitiesEditorModule, SoundUtilitiesEditor)

void FSoundUtilitiesEditorModule::StartupModule()
{
	LLM_SCOPE(ELLMTag::AudioMisc);
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	
	// Register asset actions
	AssetTools.RegisterAssetTypeActions(MakeShareable(new FAssetTypeActions_SoundSimple));

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FSoundUtilitiesEditorModule::RegisterMenus));
}

void FSoundUtilitiesEditorModule::ShutdownModule()
{
	LLM_SCOPE(ELLMTag::AudioMisc);
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner("SoundUtilities");
}

void FSoundUtilitiesEditorModule::RegisterMenus()
{
	LLM_SCOPE(ELLMTag::AudioMisc);
	FSoundWaveAssetActionExtender::RegisterMenus();
}
