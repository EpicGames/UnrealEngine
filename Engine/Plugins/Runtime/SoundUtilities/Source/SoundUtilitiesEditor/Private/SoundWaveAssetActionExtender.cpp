// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundWaveAssetActionExtender.h"
#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"

#include "AssetToolsModule.h"
#include "AssetTypeActions/AssetDefinition_SoundWave.h"
#include "ToolMenus.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserMenuContexts.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericApplication.h"
#include "Misc/PackageName.h"
#include "Sound/AudioSettings.h"
#include "Sound/SoundWaveProcedural.h"
#include "SoundSimple.h"
#include "SoundSimpleFactory.h"
#include "Algo/AnyOf.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

void FSoundWaveAssetActionExtender::RegisterMenus()
{
	if (!UToolMenus::IsToolMenuUIEnabled())
	{
		return;
	}

	FToolMenuOwnerScoped MenuOwner("SoundUtilities");

	const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(USoundWave::StaticClass());
	FToolMenuSection* Section = CastChecked<UAssetDefinition_SoundAssetBase>(AssetDefinition)->FindSoundContextMenuSection("Sound");
	check(Section);

	Section->AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		if (UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>())
		{
			if (Algo::AnyOf(Context->SelectedAssets, [](const FAssetData& AssetData){ return AssetData.IsInstanceOf<USoundWaveProcedural>(); }))
			{
				return;
			}

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&FSoundWaveAssetActionExtender::ExecuteCreateSimpleSound);
			const TAttribute<FText> Label = LOCTEXT("SoundWave_CreateSimpleSound", "Create Simple Sound");
			const TAttribute<FText> ToolTip = LOCTEXT("SoundWave_CreateSimpleSoundTooltip", "Creates a simple sound asset using the selected sound waves.");
			const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.SoundSimple");
			UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateLambda([](const FToolMenuContext&)
			{
				if (const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>())
				{
					if (AudioSettings->bEnableLegacyAssetTypes)
					{
						return true;
					}
				}
				return FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Shift);
			});
			InSection.AddMenuEntry("SoundWave_CreateSimpleSound", Label, ToolTip, Icon, UIAction);
		}
	}));
}

void FSoundWaveAssetActionExtender::ExecuteCreateSimpleSound(const FToolMenuContext& MenuContext)
{
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
	{
		const FString DefaultSuffix = TEXT("_SimpleSound");
	
		TArray<USoundWave*> SoundWaves = Context->LoadSelectedObjectsIf<USoundWave>([](const FAssetData& AssetData) { return !AssetData.IsInstanceOf<USoundWaveProcedural>(); });

		if (!SoundWaves.IsEmpty())
		{
			// Determine an appropriate name
			FString Name;
			FString PackagePath;

			FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
			AssetToolsModule.Get().CreateUniqueAssetName(SoundWaves[0]->GetOutermost()->GetName(), DefaultSuffix, PackagePath, Name);

			// Create the factory used to generate the asset
			USoundSimpleFactory* Factory = NewObject<USoundSimpleFactory>();
			Factory->SoundWaves = SoundWaves;

			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			ContentBrowserModule.Get().CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackagePath), USoundSimple::StaticClass(), Factory);
		}
	}
}

#undef LOCTEXT_NAMESPACE
