// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FSoundUtilitiesEditorModule : public IModuleInterface
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void RegisterMenus();
};
