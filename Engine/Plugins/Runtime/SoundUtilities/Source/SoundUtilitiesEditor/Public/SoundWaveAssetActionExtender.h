// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FSoundWaveAssetActionExtender
{
public:
	static void RegisterMenus();
	static void ExecuteCreateSimpleSound(const struct FToolMenuContext& MenuContext);
};

