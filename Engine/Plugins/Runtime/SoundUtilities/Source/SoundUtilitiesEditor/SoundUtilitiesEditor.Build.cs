// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SoundUtilitiesEditor : ModuleRules
	{
		public SoundUtilitiesEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"EditorFramework",
					"UnrealEd",
					"AssetDefinition",
					"AudioEditor",
					"SoundUtilities",
					"AudioMixer",
					"Slate",
					"SlateCore",
					"ContentBrowser",
					"ToolMenus",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"AssetTools",
					"AudioEditor",
				}
			);
		}
	}
}